// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealPackageTool.h"

#include "AssetRegistry/AssetRegistryState.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformMisc.h"
#include "Misc/ScopeExit.h"
#include "Serialization/StructuredArchiveFormatter.h"
#include "Serialization/Formatters/JsonArchiveOutputFormatter.h"
#include "SQLiteDatabase.h"
#include "SQLiteTypes.h"

// Undefine legacy macro that conflicts with function names in CLI11
#undef check

THIRD_PARTY_INCLUDES_START
#include "CLI/CLI.hpp"
THIRD_PARTY_INCLUDES_END

namespace UE::PackageTool
{
struct FOutput
{
	TUniquePtr<FArchive> Archive;

	FArchive* operator->()
	{
		return Archive.Get();
	}

	FArchive& operator*()
	{
		return *Archive.Get();
	}
};

struct FStructuredOutput
{
	FOutput UnstructuredOutput;
	TUniquePtr<FStructuredArchiveFormatter> Formatter;
	FStructuredArchive Writer;

	FStructuredArchive* operator->()
	{
		return &Writer;
	}

	FStructuredArchive& operator*()
	{
		return Writer;
	}
};

void WriteString(FStringView S, FArchive& Ar)
{
	auto Converted = StringCast<UTF8CHAR>(S.GetData(), S.Len());
	Ar.Serialize((void*)GetData(Converted), GetNum(Converted) * sizeof(UTF8CHAR));
}

enum class EOutputType
{
	Stdout,
	File,
	Directory
};

class FSubcommand_AssetRegistry : public FSubcommand
{
	FSharedParameters* Shared = nullptr;

	// Path to input asset registry state as a .bin file
	FString BinInput;

	EOutputType OutputType = EOutputType::Stdout;
	// Path to output file or directory
	FString OutputPath;

	bool bDump = false;

	struct FPrintOptions
	{
		bool bAssetNames = false;
		bool bTags = false;
	} PrintOptions;;

	struct FSQLiteOptions
	{
	} SQLiteOptions;

	struct FDumpOptions
	{
		bool bAll = false;
		bool bByObjectPath = false;
		bool bByPackageName = false;
		bool bByPath = false;
		bool bByClass = false;
		bool bByTag = false;
		bool bAssetTags = false;
		bool bDependencies = false;
		bool bDependencyDetails = false;
		bool bLegacyDependencies = false;
		bool bPackageData = false;
		bool bAssetBundles = false;
	} DumpOptions;

	virtual void RegisterSubcommand(FSharedParameters* InShared, CLI::App* App) override;
	void Main_Print();
	void Main_SQLite();
#if ASSET_REGISTRY_STATE_DUMPING_ENABLED
	void Main_Dump();
#endif // ASSET_REGISTRY_STATE_DUMPING_ENABLED
	
	// Helper to create output files. Filename will be used when the output type is directory
	FOutput CreateOutput(FStringView Filename);
	FStructuredOutput CreateStructuredOutput(FStringView Filename);
	
	FAssetRegistryState LoadAssetRegistryState();
};

void FSubcommand_AssetRegistry::RegisterSubcommand(FSharedParameters* InShared, CLI::App* App)
{
	Shared = InShared;
	CLI::App* Sub = App->add_subcommand("AssetRegistry", "Read asset registry bin/cache files and modify or output that data");

	auto AddInputGroup = [this](CLI::App* To)
	{
		CLI::Option_group* InputGroup = To->add_option_group("Input", "Where to get asset registry data data from");
		InputGroup->require_option(1, 1);
		InputGroup->add_option("--bin",
			"Path to a .bin file containing a saved asset registry, e.g. the output DevelopmentAssetRegistry.bin file from the cooker")
			->each([this](const std::string& s) { BinInput = ConvertPathParameter(s); });
	};

	auto AddOutputGroup = [this](CLI::App* To)
	{ 
		CLI::Option_group* OutputGroup = To->add_option_group("Output", "Where to write output data to");
		// CLI::Option_group* OutputGroup = Sub->add_option_group("Output",);
		OutputGroup->require_option(1, 1);
		OutputGroup->add_option("--out-file,-o", "Write all output to the destination file")
			->each([this](const std::string& s) 
			{
				OutputType = EOutputType::File;
				OutputPath = ConvertPathParameter(s); 
			});
		OutputGroup->add_option("--out-dir", "Write all output to files in the destination directory")
			->each([this](const std::string& s) 
			{
				OutputType = EOutputType::Directory;
				OutputPath = ConvertPathParameter(s); 
			});
		OutputGroup->add_flag("--stdout", "Write all output to standard output")->each([this](const std::string& s) 
		{
			OutputType = EOutputType::Stdout; 
		});
	};

	CLI::App* Sub_Print = Sub->add_subcommand("Print", "Print out some data from the asset registry")
		->final_callback([this]() { Main_Print(); });
	Sub_Print->add_flag("--asset-names", PrintOptions.bAssetNames, "Print all asset names");
	Sub_Print->add_flag("--tags", PrintOptions.bTags, "Print all known asset registry tags");
	Sub_Print->require_option();
	AddInputGroup(Sub_Print);
	AddOutputGroup(Sub_Print);


	CLI::App* Sub_SQLite = Sub->add_subcommand("SQLite", "Create or update an sqlite database with asset registry data")
		->final_callback([this]() { Main_SQLite(); })
		->footer("Note that this is an experimental feature and the table structure of the database is subject to change.");
	AddInputGroup(Sub_SQLite);
	
	Sub_SQLite->add_option("--out-file,-o", "Output file for the database")
	 	->required()
		->each([this](const std::string& s) 
		{
			 OutputType = EOutputType::File;
			 OutputPath = ConvertPathParameter(s); 
		});

#if ASSET_REGISTRY_STATE_DUMPING_ENABLED
	CLI::App* Sub_Dump = Sub->add_subcommand("Dump", "Dump the complete asset registry")
		->final_callback([this]() { Main_Dump(); });
	AddInputGroup(Sub_Dump);
	AddOutputGroup(Sub_Dump);

	Sub_Dump->add_flag("--all", DumpOptions.bAll,"Output all information in the asset registry");
	Sub_Dump->add_flag("--by-object-path", DumpOptions.bByObjectPath,"Output the path of each asset in the asset registry");
	Sub_Dump->add_flag("--by-package-name", DumpOptions.bByPackageName, "Output each asset in the asset registry grouped by package name");
	Sub_Dump->add_flag("--by-path", DumpOptions.bByPath, "Output each asset in the asset registry grouped by package path (package name without everything past the last /)");
	Sub_Dump->add_flag("--by-class", DumpOptions.bByClass,"Output each asset in the asset registry grouped by class");
	Sub_Dump->add_flag("--by-tag", DumpOptions.bByTag,"Output each asset in the asset registry grouped by asset tag (i.e. assets may appear more than once)");
	Sub_Dump->add_flag("--asset-tags", DumpOptions.bAssetTags,"Output each asset in the asset registry and the tags it has");
	Sub_Dump->add_flag("--dependencies", DumpOptions.bDependencies, "Output each dependency node in the asset registry without listing all dependencies from it");
	Sub_Dump->add_flag("--dependency-details", DumpOptions.bDependencyDetails,"Output each dependency node in the asset registry and list all basic dependencies from it");
	Sub_Dump->add_flag("--legacy-dependencies", DumpOptions.bLegacyDependencies, "Output each dependency node in the asset registry and list all detailed dependencies from it");
	Sub_Dump->add_flag("--package-data", DumpOptions.bPackageData,"Output information about each package file in the asset registry");
	Sub_Dump->add_flag("--asset-bundles", DumpOptions.bAssetBundles,"Output the contents of each asset bundle in the asset registry");
#endif // ASSET_REGISTRY_STATE_DUMPING_ENABLED
}

FOutput FSubcommand_AssetRegistry::CreateOutput(FStringView Filename)
{
	IFileManager& FM = IFileManager::Get();
	TUniquePtr<FArchive> Output;
	switch (OutputType)
	{
	case EOutputType::Stdout:
		Output.Reset(new FArchiveStdOut);
		break;
	case EOutputType::File:
		{
			Output.Reset(FM.CreateFileWriter(*OutputPath));
		}
		break;
	case EOutputType::Directory:
		{
			if (!FM.DirectoryExists(*OutputPath))
			{
				if (!FM.MakeDirectory(*OutputPath, true))
				{
					throw CLI::Error("FailedToCreateOutDirectory", "Failed to create output directory", 1);
				}
			}

			TStringBuilder<512> FilePath;
			FPathViews::Append(FilePath, OutputPath, Filename);
			Output.Reset(FM.CreateFileWriter(*FilePath));
		}
		break;
	}

	if (!Output.Get())
	{
		throw CLI::Error("FailedToOpenOutFile", "Failed to open output file", 1);
	}

	return FOutput{ MoveTemp(Output) };
}

FStructuredOutput FSubcommand_AssetRegistry::CreateStructuredOutput(FStringView Filename)
{
	FOutput Output = CreateOutput(Filename);
	TUniquePtr<FStructuredArchiveFormatter> Formatter;
	if (Shared->bJSON)
	{
		Formatter.Reset(new FJsonArchiveOutputFormatter(*Output));	
	}
	else
	{
		Formatter.Reset(new FTextOutputFormatter(*Output));	
	}

	FStructuredArchiveFormatter* FormatterPtr = Formatter.Get();
	return FStructuredOutput{ MoveTemp(Output), MoveTemp(Formatter), FStructuredArchive(*FormatterPtr) };
}

FAssetRegistryState FSubcommand_AssetRegistry::LoadAssetRegistryState()
{
	FAssetRegistryState State;
	FAssetRegistryLoadOptions LoadOptions;
	LoadOptions.ParallelWorkers = FPlatformMisc::NumberOfCoresIncludingHyperthreads();
	FAssetRegistryVersion::Type Version = FAssetRegistryVersion::PreVersioning;
	if (!BinInput.IsEmpty())
	{
		if (!FAssetRegistryState::LoadFromDisk(*BinInput, LoadOptions, State, &Version))
		{
			throw CLI::Error("FailedToLoadAssetRegistry", "Failed to load asset registry state", 1);
		}
	}

	return MoveTemp(State);
}

void FSubcommand_AssetRegistry::Main_SQLite()
{
	// Re-enable some logging since we aren't using stdout
	FSelfRegisteringExec::StaticExec(nullptr, TEXT("log reset"), *GLog);

	IFileManager& FM = IFileManager::Get();
	if (FM.FileExists(*OutputPath))
	{
		FM.Delete(*OutputPath);
	}

	FSQLiteDatabase DB;
	DB.Open(*OutputPath);

	if (!DB.IsValid())
	{
		throw CLI::Error("FailedToOpenDB", "Failed to open database");
	}

	ON_SCOPE_EXIT
	{
		DB.Close();
	};

	// don't bother waiting for disk to flush (let the OS handle it)
	DB.Execute(TEXT("PRAGMA synchronous=OFF;"));
	// 1000 page cache
	DB.Execute(TEXT("PRAGMA cache_size=1000;"));
	// large-ish page sizes for modern disks
	DB.Execute(TEXT("PRAGMA page_size=65535;"));
	// hold the file lock the whole time.
	DB.Execute(TEXT("PRAGMA locking_mode=EXCLUSIVE;"));
	// Hold things in memory unti commit(?)
	DB.Execute(TEXT("PRAGMA temp_store=MEMORY;"));

	DB.Execute(
		TEXT("CREATE TABLE AssetClasses (")
		TEXT("PackageName TEXT, ")
		TEXT("ClassName TEXT, ")
		TEXT("UNIQUE (PackageName, ClassName) ")
		TEXT(");")
	);

	DB.Execute(
		TEXT("CREATE TABLE Assets (")
		TEXT("PackageName TEXT, ")
		TEXT("AssetName TEXT, ")
		TEXT("AssetClassID INTEGER, ")
		TEXT("OptionalOuterPath TEXT, ")
		TEXT("PackageFlags INTEGER, ") // TODO: Move to a separate table? 
		TEXT("FOREIGN KEY(AssetClassID) REFERENCES AssetClasses(ROWID), ")
		TEXT("UNIQUE (PackageName, AssetName) ")
		TEXT(");")
	);

	DB.Execute(
		TEXT("CREATE TABLE Tags (")
		TEXT("TagName TEXT, ")
		TEXT("UNIQUE (TagName) ")
		TEXT(");")
	);

	DB.Execute(
		TEXT("CREATE TABLE AssetTags (")
		TEXT("AssetID INTEGER, ")
		TEXT("TagID INTEGER, ")
		TEXT("TagValue TEXT, ")
		TEXT("FOREIGN KEY(AssetID) REFERENCES Assets(ROWID), ")
		TEXT("FOREIGN KEY(TagID) REFERENCES Tags(ROWID), ")
		TEXT("UNIQUE(AssetID, TagID) ")
		TEXT(");")
	);

	DB.Execute(
		TEXT("CREATE TEMPORARY TABLE AssetsImport( ")
		TEXT("PackageName, AssetName, ClassPackageName, ClassName, OptionalOuterPath, PackageFlags, ")
		TEXT("UNIQUE(PackageName, AssetName) ")
		TEXT(");")
	);

	DB.Execute(
		TEXT("CREATE TEMPORARY TABLE TagsImport( ")
		TEXT("PackageName, AssetName, TagName, TagValue, ")
		TEXT("UNIQUE(PackageName, AssetName, TagName) ")
		TEXT(");")
	);

	FSQLitePreparedStatement InsertClass;
	const TCHAR* InsertClassText = TEXT("INSERT INTO AssetClasses (PackageName, ClassName) VALUES (?1, ?2);");
	if (!InsertClass.Create(DB, InsertClassText))
	{
		throw CLI::Error("FailedToCreatePreparedStatement", "Failed to create prepared statement", 1);
	}

	FSQLitePreparedStatement InsertAsset;
	const TCHAR* InsertAssetText = 
		TEXT("INSERT INTO AssetsImport (PackageName, AssetName, ClassPackageName, ClassName, OptionalOuterPath, PackageFlags) ")
		TEXT("VALUES (?1, ?2, ?3, ?4, ?5, ?6);");
	if (!InsertAsset.Create(DB, InsertAssetText))
	{
		throw CLI::Error("FailedToCreatePreparedStatement", "Failed to create prepared statement", 1);
	}

	FSQLitePreparedStatement InsertTag;
	const TCHAR* InsertTagText = TEXT("INSERT INTO TagsImport (PackageName, AssetName, TagName, TagValue) VALUES (?1, ?2, ?3, ?4);");
	if (!InsertTag.Create(DB, InsertTagText))
	{
		throw CLI::Error("FailedToCreatePreparedStatement", "Failed to create prepared statement", 1);
	}


	const TCHAR* FinalizeAssets = 
	 	// Populate Assets table referencing AssetClasses
		TEXT("INSERT INTO Assets (PackageName, AssetName, AssetClassID, OptionalOuterPath, PackageFlags) ")
		TEXT("SELECT AssetsImport.PackageName, AssetsImport.AssetName, AssetClasses.ROWID, AssetsImport.OptionalOuterPath, AssetsImport.PackageFlags ")
		TEXT("FROM AssetsImport ")
		TEXT("INNER JOIN AssetClasses ON AssetClasses.PackageName == AssetsImport.ClassPackageName AND AssetClasses.ClassName == AssetsImport.ClassName; ");
	const TCHAR* FinalizeTags = 
		// Populate Tags
		TEXT("INSERT INTO Tags (TagName) SELECT DISTINCT TagName FROM TagsImport; ");
	const TCHAR* FinalizeAssetTags =
		// Populate AssetTags referencing Assets and Tags
		TEXT("INSERT INTO AssetTags (AssetID, TagID, TagValue) ")	
		TEXT("SELECT Assets.ROWID, Tags.RowID, TagsImport.TagValue ")
		TEXT("FROM TagsImport ")
		TEXT("INNER JOIN Assets ON Assets.PackageName == TagsImport.PackageName AND Assets.AssetName == TagsImport.AssetName ")
		TEXT("INNER JOIN Tags ON Tags.TagName == TagsImport.TagName; ");

	FAssetRegistryState State = LoadAssetRegistryState();

	TSet<FTopLevelAssetPath> AssetClasses;
	State.EnumerateAllAssets([&](const FAssetData& AssetData) 
	{
		AssetClasses.Add(AssetData.AssetClassPath);
	});


	DB.Execute(TEXT("BEGIN TRANSACTION;"));
	int32 i = 0;
	constexpr int32 BatchSize = 10000;
	for (FTopLevelAssetPath Class : AssetClasses)
	{
		InsertClass.Reset();
		InsertClass.ClearBindings();
		InsertClass.SetBindingValueByIndex(1, Class.GetPackageName().ToString());
		InsertClass.SetBindingValueByIndex(2, Class.GetAssetName().ToString());

		if (!InsertClass.Execute())
		{
			throw CLI::Error("FailedToInsertAssetClass", "Failed to insert asset class", 1);
		}

		if (++i == BatchSize)
		{
			i = 0;
			DB.Execute(TEXT("END TRANSACTION;"));
			DB.Execute(TEXT("BEGIN TRANSACTION;"));
		}

	}
	DB.Execute(TEXT("END TRANSACTION;"));


	i = 0;
	DB.Execute(TEXT("BEGIN TRANSACTION;"));
	State.EnumerateAllAssets([&](const FAssetData& AssetData) 
	{
		InsertAsset.Reset();
		InsertAsset.ClearBindings();
		InsertAsset.SetBindingValueByIndex(1, AssetData.PackageName.ToString());
		InsertAsset.SetBindingValueByIndex(2, AssetData.AssetName.ToString());
		InsertAsset.SetBindingValueByIndex(3, AssetData.AssetClassPath.GetPackageName().ToString());
		InsertAsset.SetBindingValueByIndex(4, AssetData.AssetClassPath.GetAssetName().ToString());
		InsertAsset.SetBindingValueByIndex(5, AssetData.GetOptionalOuterPathName().ToString());
		InsertAsset.SetBindingValueByIndex(6, AssetData.PackageFlags);

		if (!InsertAsset.Execute())
		{
			throw CLI::Error("FailedToInsertAsset", "Failed to insert asset", 1);
		}

		if (++i == BatchSize)
		{
			i = 0;
			DB.Execute(TEXT("END TRANSACTION;"));
			DB.Execute(TEXT("BEGIN TRANSACTION;"));
		}
	});
	DB.Execute(TEXT("END TRANSACTION;"));

	i = 0;
	DB.Execute(TEXT("BEGIN TRANSACTION;"));
	State.EnumerateAllAssets([&](const FAssetData& AssetData) 
	{
		for (TPair<FName, FAssetTagValueRef> Tag : AssetData.TagsAndValues)
		{
			InsertTag.Reset();
			InsertTag.ClearBindings();
			InsertTag.SetBindingValueByIndex(1, AssetData.PackageName.ToString());
			InsertTag.SetBindingValueByIndex(2, AssetData.AssetName.ToString());
			InsertTag.SetBindingValueByIndex(3, Tag.Key.ToString());
			InsertTag.SetBindingValueByIndex(4, Tag.Value.AsString());

			if (!InsertTag.Execute())
			{
				throw CLI::Error("FailedToInsertTag", "Failed to insert tag", 1);
			}

			if (++i == BatchSize)
			{
				i = 0;
				DB.Execute(TEXT("END TRANSACTION;"));
				DB.Execute(TEXT("BEGIN TRANSACTION;"));
			}
		}
	});
	DB.Execute(TEXT("END TRANSACTION;"));

	DB.Execute(FinalizeAssets);
	DB.Execute(FinalizeTags);
	DB.Execute(FinalizeAssetTags);
}

void FSubcommand_AssetRegistry::Main_Print()
{
	FAssetRegistryState State = LoadAssetRegistryState();

	FStructuredOutput Output = CreateStructuredOutput(TEXTVIEW("AssetRegistryData.txt"));
	FStructuredArchive::FRecord Root = Output->Open().EnterRecord();
	if (PrintOptions.bAssetNames)
	{
		FStructuredArchive::FStream Array = Root.EnterField(TEXT("AssetNames")).EnterStream();
		State.EnumerateAllAssets([&](const FAssetData& AssetData) 
		{
			FSoftObjectPath Path = AssetData.GetSoftObjectPath();
			Array << Path;
			return true;
		});
	}
	if (PrintOptions.bTags)
	{
		FStructuredArchive::FStream Array = Root.EnterField(TEXT("Tags")).EnterStream();
		State.EnumerateTags([&](FName TagName) 
		{
			Array << TagName;
			return true;
		});
	}

}

#if ASSET_REGISTRY_STATE_DUMPING_ENABLED
void FSubcommand_AssetRegistry::Main_Dump()
{
	FAssetRegistryState State = LoadAssetRegistryState();

	TArray<FString> Pages;
	
	// TODO: Move args directly to option parsing
	TArray<FString> Args;
	if (DumpOptions.bAll)
	{
		Args.Emplace(TEXT("All"));
	}

	if (DumpOptions.bByObjectPath)
	{
		Args.Emplace(TEXT("ObjectPath"));
	}

	if (DumpOptions.bByPackageName)
	{
		Args.Emplace(TEXT("PackageName"));
	}

	if (DumpOptions.bByPath)
	{
		Args.Emplace(TEXT("Path"));
	}

	if (DumpOptions.bByClass)
	{
		Args.Emplace(TEXT("Class"));
	}

	if (DumpOptions.bByTag)
	{
		Args.Emplace(TEXT("Tag"));
	}

	if (DumpOptions.bAssetTags)
	{
		Args.Emplace(TEXT("AssetTags"));
	}

	if (DumpOptions.bDependencies)
	{
		Args.Emplace(TEXT("Dependencies"));
	}

	if (DumpOptions.bDependencyDetails)
	{
		Args.Emplace(TEXT("DependencyDetails"));
	}

	if (DumpOptions.bLegacyDependencies)
	{
		Args.Emplace(TEXT("LegacyDependencies"));
	}

	if (DumpOptions.bPackageData)
	{
		Args.Emplace(TEXT("PackageData"));
	}

	if (DumpOptions.bAssetBundles)
	{
		Args.Emplace(TEXT("AssetBundles"));
	}

	State.Dump(Args, Pages, 10000);
	for (int32 i=0; i < Pages.Num(); ++i)
	{
		FOutput Output = CreateOutput(WriteToString<256>(TEXT("AssetRegistryDump_"), i, TEXT(".txt")));
		WriteString(Pages[i], *Output.Archive);
	}
}
#endif // ASSET_REGISTRY_STATE_DUMPING_ENABLED

FSubcommand_AssetRegistry GSubcommand_AssetRegistry;

}