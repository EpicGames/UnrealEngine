// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealPackageTool.h"
#include "Misc/FileHelper.h"
#include "UObject/NameBatchSerialization.h"

// Undefine legacy macro that conflicts with function names in CLI11
#undef check

THIRD_PARTY_INCLUDES_START
#include "CLI/CLI.hpp"
THIRD_PARTY_INCLUDES_END

namespace UE::PackageTool
{

class FSubcommand_Names : public FSubcommand
{
	FSharedParameters* Shared = nullptr;

	virtual void RegisterSubcommand(FSharedParameters* InShared, CLI::App* App) override;
	
	FString InputFilename;
	FString OutputFilename;
	
#if ALLOW_NAME_BATCH_SAVING
	void Main_CreateBatch();
#endif // ALLOW_NAME_BATCH_SAVING

	void Main_ReadBatch();
};

void FSubcommand_Names::RegisterSubcommand(FSharedParameters* InShared, CLI::App* App)
{
	Shared = InShared;

	CLI::App* Sub_Names = App->add_subcommand("Names",
		"Commands operating on FNames e.g. reading and writing binary files containing prepared batches of FNames for loading")
		->require_subcommand();

#if ALLOW_NAME_BATCH_SAVING
	CLI::App* Sub_CreateBatch = Sub_Names->add_subcommand("CreateBatch", "Create a binary file containing a batch of names for loading")
		->final_callback([this]() { Main_CreateBatch(); });
	Sub_CreateBatch->add_option("--input,-i", "Input text file containing one name per line")
		->each([this](const std::string& s) { InputFilename = ConvertPathParameter(s); })
		->required();
	Sub_CreateBatch->add_option("--output,-o", "Output file path")
		->each([this](const std::string& s) { OutputFilename = ConvertPathParameter(s); })
		->required();
#endif // ALLOW_NAME_BATCH_SAVING

	CLI::App* Sub_ReadBatch = Sub_Names->add_subcommand("ReadBatch", "Read a binary file containing a batch of names")
		->final_callback([this]() { Main_ReadBatch(); });
	Sub_ReadBatch->add_option("--input,-i", "Input binary file containing name batch")
		->each([this](const std::string& s) { InputFilename = ConvertPathParameter(s); })
		->required();
	Sub_ReadBatch->add_option("--output,-o", "Output file path")
		->each([this](const std::string& s) { OutputFilename = ConvertPathParameter(s); });
}

#if ALLOW_NAME_BATCH_SAVING
void FSubcommand_Names::Main_CreateBatch()
{
	TArray<FString> Lines;
	if (!FFileHelper::LoadFileToStringArray(Lines, *InputFilename))
	{
		throw CLI::Error("FailedToReadInputFile", "Failed to read input file", 1);
	}

	TUniquePtr<FArchive> OutArchive(IFileManager::Get().CreateFileWriter(*OutputFilename));
	if (!OutArchive)
	{
		throw CLI::Error("FailedToOpenOutputFile", "Failed to open output file", 1);
	}

	TArray<FDisplayNameEntryId> Names;
	Names.Reserve(Lines.Num());
	for (const FString& Line : Lines)
	{
		FName Name(*Line, NAME_NO_NUMBER_INTERNAL, false);
		Names.Emplace(Name);
	}
	SaveNameBatch(MakeArrayView(Names), *OutArchive);
	if (OutArchive->IsError())
	{
		throw CLI::Error("FailedToWriteOutputFile", "Failed to write output file", 1);
	}
}
#endif // ALLOW_NAME_BATCH_SAVING

void FSubcommand_Names::Main_ReadBatch()
{
	TUniquePtr<FArchive> InArchive(IFileManager::Get().CreateFileReader(*InputFilename));
	if (!InArchive)
	{
		throw CLI::Error("FailedToReadInputFile", "Failed to read input file", 1);
	}

	IFileManager& FM = IFileManager::Get();
	TUniquePtr<FArchive> Output;
	if (OutputFilename.IsEmpty())
	{
		Output.Reset(new FArchiveStdOut);
	}
	else
	{
		Output.Reset(FM.CreateFileWriter(*OutputFilename));
		if (!Output)
		{
			throw CLI::Error("FailedToOpenOutputFile", "Failed to open output file", 1);
		}
	}

	TArray<FDisplayNameEntryId> Names = LoadNameBatch(*InArchive);
	for (FDisplayNameEntryId Name : Names)
	{
		TUtf8StringBuilder<FName::StringBufferSize> Buffer(InPlace, Name.ToName(NAME_NO_NUMBER_INTERNAL), "\n");
		Output->Serialize((void*)GetData(Buffer), GetNum(Buffer) * sizeof(UTF8CHAR));
	}
}

FSubcommand_Names GSubcommand_Names;

}