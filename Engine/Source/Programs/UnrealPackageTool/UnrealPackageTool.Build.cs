// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UnrealPackageTool : ModuleRules
{
	public UnrealPackageTool(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore", // AssetRegistry requires this 
				"Core",
				"CoreUObject",
				"Projects",
				"SQLiteCore",
				"AssetRegistry"
			}
		);

		bUseUnity = false;
		PrivateDependencyModuleNames.Add("CLI11");
		
		bEnableExceptions = true;
		bTreatAsEngineModule = true; // So we can use internal headers
	}
}
