// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class UnrealPackageToolTarget : TargetRules
{
	public UnrealPackageToolTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LaunchModuleName = "UnrealPackageTool";
		DefaultBuildSettings = BuildSettingsVersion.Latest;

		// Required for CLI11 library 
		bForceEnableRTTI = true;
		bForceEnableExceptions = true;

		bBuildDeveloperTools = false;
		bBuildWithEditorOnlyData = true;

		// Required for asset registry module
		bCompileAgainstApplicationCore = true;
		bCompileAgainstCoreUObject = true;
		bCompileAgainstEngine = false;
		bCompileICU = false;

		bIsBuildingConsoleApplication = true;

		bEnableTrace = true;
	}
}
