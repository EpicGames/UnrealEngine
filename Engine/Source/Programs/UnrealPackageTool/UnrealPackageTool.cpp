// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealPackageTool.h"

#include "Algo/MaxElement.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/PackageReader.h"
#include "Async/ParallelFor.h"
#include "Containers/Array.h"
#include "Containers/ConsumeAllMpmcQueue.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFile.h"
#include "Misc/CommandLine.h"
#include "Misc/ObjectThumbnail.h"
#include "Misc/PackagePath.h"
#include "Misc/Parse.h"
#include "Misc/WildcardString.h"
#include "Modules/ModuleManager.h"
#include "RequiredProgramMainCPPInclude.h"
#include "Serialization/ArchiveProxy.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/StructuredArchiveFormatter.h"
#include "Serialization/Formatters/JsonArchiveOutputFormatter.h"
#include "String/ParseTokens.h"
#include "UObject/PackageFileSummary.h"

#if PLATFORM_WINDOWS
#include <io.h>
#include <fcntl.h>
#endif

// Undefine legacy macro that conflicts with function names in CLI11
#undef check

THIRD_PARTY_INCLUDES_START
#include "CLI/CLI.hpp"
THIRD_PARTY_INCLUDES_END

DEFINE_LOG_CATEGORY_STATIC(LogUnrealPackageTool, Log, All);


// These macros are not properly defined by UBT in the case of an engine program with bTreatAsEngineModule=true
// So define them here as a workaround
#define IMPLEMENT_ENCRYPTION_KEY_REGISTRATION()
#define IMPLEMENT_SIGNING_KEY_REGISTRATION()
IMPLEMENT_APPLICATION(UnrealPackageTool, "UnrealPackageTool");

namespace UE::PackageTool
{

FSubcommand* SubcommandList = nullptr;

FSubcommand::FSubcommand()
{
	Next = SubcommandList;
	SubcommandList = this;
}

FSubcommand::~FSubcommand()
{

}
	
	
// Convert a utf8 command line parameter to an FString path, treating relative paths as relative to the working directory rather than the engine base dir
FString ConvertPathParameter(const std::string& InParam)
{
	FString Path = UTF8_TO_TCHAR(InParam.c_str());
	if (FPaths::IsRelative(Path))
	{
		FString CWD = FPlatformMisc::LaunchDir();
		Path = CWD / Path;
		return FPaths::ConvertRelativePathToFull(MoveTemp(Path));
	}
	return Path;
}

// Parameters shared between multiple execution modes
FSharedParameters::FSharedParameters(CLI::App* InApp)
{
	InApp->add_flag("--json,-j", bJSON, "Output structured data in JSON format. Without this flag, data is intended to be human readable and not reliably parseable.")
		->trigger_on_parse(); // Required to allow parsing before subcommands
}

// Utility archive for printing json/package info to stdout
FArchiveStdOut::~FArchiveStdOut()
{
	fflush(stdout);
}

// Both formatter types provide utf8 text
void FArchiveStdOut::Serialize(void* Data, int64 Len) 
{
#if PLATFORM_WINDOWS
	// replace \r with a space to avoid CRT printf's function expanding \r\n to \r\r\n
	for (UTF8CHAR* C = (UTF8CHAR*)Data, *End = C + Len / sizeof(UTF8CHAR); C != End; ++C)
	{
		if (*C == '\r')
		{
			*C = (UTF8CHAR)' ';
		}
	}
	auto Converted = StringCast<WIDECHAR>((const UTF8CHAR*)Data, Len / sizeof(UTF8CHAR));
	wprintf(TEXT("%.*s"), (int)(Converted.Length()), Converted.Get());
	Pos += Converted.Length() * sizeof(TCHAR);
#else
	printf("%.*s", (int)(Len / sizeof(char)), (const char*)Data);
	Pos += Len;
#endif		
}

// Required for correct formatting of JSON output
int64 FArchiveStdOut::Tell() 
{
	return Pos;
}

TArray<FString> GatherAssetsInPaths(TConstArrayView<FString> Roots, bool bRecursive)
{
	TConsumeAllMpmcQueue<FString> PackagePaths;
	class FPackageRootVisitor final : public IPlatformFile::FDirectoryVisitor
	{
		bool bRecursive;
		TConsumeAllMpmcQueue<FString>& PackagePaths;
	public:
		FPackageRootVisitor(bool InRecursive, TConsumeAllMpmcQueue<FString>& InPackagePaths)
			: IPlatformFile::FDirectoryVisitor(EDirectoryVisitorFlags::ThreadSafe)
			, bRecursive(InRecursive)
			, PackagePaths(InPackagePaths)
		{
		}

		bool Visit(const TCHAR* Path, bool bIsDirectory) final
		{
			if (!bIsDirectory)
			{
				EPackageExtension Extension = FPackagePath::ParseExtension(Path);
				if (Extension == EPackageExtension::Asset || Extension == EPackageExtension::Map)
				{
					PackagePaths.ProduceItem(Path);
				}
				return true;
			}
			return bRecursive;
		}

	};
	
	FPackageRootVisitor Visitor(bRecursive, PackagePaths);
	ParallelFor(TEXT("GatherAssestInPaths"), Roots.Num(), 1, [&PackagePaths, &Roots, &Visitor](int32 Index)
	{
		IFileManager& FM = IFileManager::Get();
		const FString& Path = Roots[Index];
		if (FM.FileExists(*Path))
		{
			PackagePaths.ProduceItem(Path);
		}
		else if (FM.DirectoryExists(*Path))
		{
			IFileManager::Get().IterateDirectoryRecursively(*Path, Visitor);
		}
		else
		{
			UE_LOGF(LogUnrealPackageTool, Error, "Input path %ls is neither a file nor directory", *Path);
		}
		
	}, EParallelForFlags::Unbalanced);

	TArray<FString> OutPaths;
	PackagePaths.ConsumeAllLifo([&OutPaths](FString PackagePath)
	{
		OutPaths.Emplace(MoveTemp(PackagePath));
	});

	return OutPaths;
}

struct FSubcommand_LicenseeVersionIsError
{	
	FSharedParameters& Shared;
	TArray<FString> PackageRoots;

	FSubcommand_LicenseeVersionIsError(FSharedParameters& InShared, CLI::App* InApp)
		: Shared(InShared)
	{
		CLI::App* Sub = InApp->add_subcommand("LicenseeVersionIsError", "Ensure that no assets have a licensee version set.")
			->fallthrough()
			->preparse_callback([this](std::size_t)
			{
				PackageRoots.Reset();
			});
		Sub->add_option("--AllPackagesIn,-d", "Check all packages in the given directories")
			->required()
			->multi_option_policy(CLI::MultiOptionPolicy::TakeAll)
			->each([this](const std::string& s) { PackageRoots.Emplace(ConvertPathParameter(s)); })
			->check(CLI::ExistingDirectory);
		Sub->parse_complete_callback([this]() { Main(); });
	}
	
	void Main()
	{
		UE_LOGF(LogUnrealPackageTool, Log, "Checking packages for licensee version");

		TArray<FString> PackagePaths = GatherAssetsInPaths(PackageRoots, true);

		ParallelFor(TEXT("ScanPackage.PF"), PackagePaths.Num(), 1, [this, &PackagePaths](int32 Index)
		{
			ScanPackage(*PackagePaths[Index]);
		}, EParallelForFlags::Unbalanced);
	}

	void ScanPackage(const TCHAR* Path)
	{
		if (TUniquePtr<FArchive> Ar{IFileManager::Get().CreateFileReader(Path, FILEREAD_Silent)})
		{
			UE_LOGF(LogUnrealPackageTool, Log, "Scanning package %ls", Path);
			FPackageFileSummary Summary;
			*Ar << Summary;
			if (Ar->Close())
			{
				if (Summary.CompatibleWithEngineVersion.IsLicenseeVersion())
				{
					UE_LOGF(LogUnrealPackageTool, Error, "Package has a licensee version: %ls", Path);
				}
			}
			else
			{
				UE_LOGF(LogUnrealPackageTool, Warning, "Failed to read package file summary: %ls", Path);
			}
		}
		else
		{
			UE_LOGF(LogUnrealPackageTool, Warning, "Failed to open package: %ls", Path);
		}
	}
};

int32 FTextOutputFormatter::GetIndent()
{
	if (Stack.Num() == 0)
	{
		return -1; // Prevent top-level record from being indented
	}
	return Stack.Top().Indent;
}

FTextOutputFormatter::FBuffer& FTextOutputFormatter::GetValueBuffer()
{
	checkf(Stack.Num() && Stack.Top().ValueBuffer.Get(), TEXT("FTextOutputFormatter: Accessing Value buffer without a scope to output to"));
	return *Stack.Top().ValueBuffer;
}

void FTextOutputFormatter::FlushBuffer(FBuffer& Buffer)
{
	Ar.Serialize((void*)Buffer.GetData(), Buffer.Len() * sizeof(FBuffer::ElementType));
	Buffer.Reset();
}

bool FTextOutputFormatter::LooksLikeBinary(FStringView View)
{
	for (const TCHAR& C : View)
	{
		// Treat strings containing low-ascii characters as binary
		if (C < ' ')
		{
			return true;
		}	
	}
	return false;
}

bool FTextOutputFormatter::CharNeedsEscaping(TCHAR C)
{
	switch (C)
	{
	case '\"':
	case '\\':
	case '\b':
	case '\f':
	case '\n':
	case '\r':
	case '\t':
		return true;
	default:
		if (C < ' ')
		{
			return true;
		}	
	}
	return false;
}

bool FTextOutputFormatter::NeedsEscaping(FStringView View)
{
	for (const TCHAR& C : View)
	{
		if (CharNeedsEscaping(C))
		{
			return true;
		}
	}
	return false;
}
void FTextOutputFormatter::WriteEscaped(FBuffer& Buffer, FStringView View)
{
	while(!View.IsEmpty())
	{
		bool bEscape = CharNeedsEscaping(View[0]);
		int32 Count = 1;
		for (; Count < View.Len() && bEscape == CharNeedsEscaping(View[Count]); ++Count)
		{
		}
		
		FStringView Section = View.Left(Count);
		View.RightChopInline(Count);
		if (bEscape)
		{
			for (TCHAR C : Section)
			{
				switch (C)
				{
				case '\"': Buffer << UTF8TEXTVIEW("\""); break;
				case '\\': Buffer << UTF8TEXTVIEW("\\"); break;
				case '\b': Buffer << UTF8TEXTVIEW("\\b"); break;
				case '\f': Buffer << UTF8TEXTVIEW("\\f"); break;
				case '\n': Buffer << UTF8TEXTVIEW("\\n"); break;
				case '\r': Buffer << UTF8TEXTVIEW("\\r"); break;
				case '\t': Buffer << UTF8TEXTVIEW("\\t"); break;
				default:
					if (C < ' ')
					{
						Buffer.Appendf("\\u%04x", C);
					}	
					else
					{
						Buffer.AppendChar(C);
					}
				}
			}	
		}
		else
		{
			Buffer << Section;
		}
	}

}

FTextOutputFormatter::FTextOutputFormatter(FArchive& InAr)
	: Ar(InAr)
{
	Ar.SetIsTextFormat(true);
}

FTextOutputFormatter::~FTextOutputFormatter()
{
}

FArchive& FTextOutputFormatter::GetUnderlyingArchive()
{
	return Ar;
}

bool FTextOutputFormatter::HasDocumentTree() const
{
	return true;
}

// Records: accumulate field-value pairs and write out so that key names are aligned together
void FTextOutputFormatter::EnterRecord()
{
	if (Stack.Num() == 0)
	{
		Stack.Push(FStackEntry{ GetIndent() });				
	}
	else
	{
		Stack.Push(FStackEntry{ GetIndent()+1 });				
	}
}
void FTextOutputFormatter::LeaveRecord()
{
	FStackEntry Record = Stack.Pop(EAllowShrinking::No);
	if (Record.MapEntries.Num() == 0)
	{
		return;
	}

	if (Stack.Num() == 0)
	{
		// Nothing
	}
	else 
	{
		FBuffer& Buffer = GetValueBuffer();
		TTuple<FString, FString>* Longest = Algo::MaxElementBy(Record.MapEntries, [](const TTuple<FString, FString>& Entry) { return Entry.Key.Len(); });
		int32 LongestKeyLen = Longest->Key.Len();
		int32 ColumnSize = LongestKeyLen + 4; 
		for (const TTuple<FString, FString>& KV : Record.MapEntries)
		{
			Buffer.Appendf(LINE_TERMINATOR_ANSI "%*s",
				Record.Indent * IndentWidth, "");
			Buffer << KV.Key;
			Buffer.Appendf("%-*s",
				ColumnSize - KV.Key.Len(), ":");
			Buffer << KV.Value;
			if (Stack.Num() == 1)
			{
				FlushBuffer(GetValueBuffer());
			}
		}	
		if (Stack.Num() == 1)
		{
			GetValueBuffer() << LINE_TERMINATOR_ANSI << LINE_TERMINATOR_ANSI;
			FlushBuffer(GetValueBuffer());
		}
	}
}
void FTextOutputFormatter::EnterField(FArchiveFieldName Name)
{
	if (Stack.Num() == 1 )
	{
		// Top level fields should be written as major headings 
		// Then the contents of them are _not_ indented
		Stack.Top().ValueBuffer = MakeUnique<FBuffer>();
		GetValueBuffer() << UTF8TEXTVIEW("# ") << Name.Name << LINE_TERMINATOR_ANSI "---";
	}
	else
	{
		FString ElemName = Name.Name;
		EnterMapElement(ElemName);
	}
}
void FTextOutputFormatter::LeaveField()
{
	if (Stack.Num() != 1)
	{
		LeaveMapElement();
	}
}
bool FTextOutputFormatter::TryEnterField(FArchiveFieldName Name, bool bEnterWhenSaving)
{
	EnterField(Name);
	return true;
}

void FTextOutputFormatter::EnterArray(int32& NumElements)
{
	if (NumElements == 0)
	{	
		GetValueBuffer() << UTF8TEXTVIEW("Empty");
	}
	EnterStream();
}
void FTextOutputFormatter::LeaveArray()
{
	LeaveStream();
}
void FTextOutputFormatter::EnterArrayElement()
{
	EnterStreamElement();
}
void FTextOutputFormatter::LeaveArrayElement()
{
	LeaveStreamElement();
}

void FTextOutputFormatter::EnterStream()
{
	if (Stack.Num() == 0)
	{
		checkf(false, TEXT("Top level streams are unsupported"));
	}
	Stack.Push(FStackEntry{ GetIndent() + 1 });	
}
void FTextOutputFormatter::LeaveStream()
{
	if (Stack.Top().ValueBuffer)
	{
		LeaveStreamElement();
	}
	FStackEntry Record = Stack.Pop(EAllowShrinking::No);
	if (Stack.Num() == 1)
	{
		GetValueBuffer() << LINE_TERMINATOR << LINE_TERMINATOR;
		FlushBuffer(GetValueBuffer());
	}
}
void FTextOutputFormatter::EnterStreamElement()
{
	// Array elements written prefixed with '-' at current indendation level, no other alignment 
	Stack.Top().ValueBuffer = MakeUnique<FBuffer>();	
	GetValueBuffer().Appendf(LINE_TERMINATOR_ANSI "%*s- ", Stack.Top().Indent * IndentWidth, "");
}
void FTextOutputFormatter::LeaveStreamElement()
{
	// Write value to parent scope so parent scope can do the alignment it wants 
	FStackEntry& Parent = Stack[Stack.Num()-2];
	FStackEntry& This = Stack.Top();
	(*Parent.ValueBuffer) << This.ValueBuffer->ToView();
	This.ValueBuffer.Reset();

	// If parent scope is top level, flush
	if (Stack.Num() == 2)
	{
		FlushBuffer(*Parent.ValueBuffer);
	}
}

void FTextOutputFormatter::EnterMap(int32& NumElements)
{
	if (NumElements == 0)
	{
		GetValueBuffer() << UTF8TEXTVIEW("Empty");
	}
	EnterRecord();
}
void FTextOutputFormatter::LeaveMap()
{
	LeaveRecord();
}
void FTextOutputFormatter::EnterMapElement(FString& Name)
{
	// Map keys should be aligned like records
	FStackEntry& Top = Stack.Top();	
	TTuple<FString, FString>& Entry = Top.MapEntries.AddDefaulted_GetRef();
	Entry.Key = Name;
	Top.ValueBuffer = MakeUnique<FBuffer>();
}
void FTextOutputFormatter::LeaveMapElement()
{
	TUniquePtr<FBuffer> Value = MoveTemp(Stack.Top().ValueBuffer);
	Stack.Top().MapEntries.Last().Value = Value->ToString();
}

void FTextOutputFormatter::EnterAttributedValue()
{
}
void FTextOutputFormatter::EnterAttribute(FArchiveFieldName AttributeName)
{
}
void FTextOutputFormatter::EnterAttributedValueValue()
{
}
void FTextOutputFormatter::LeaveAttribute()
{
}
void FTextOutputFormatter::LeaveAttributedValue()
{
}
bool FTextOutputFormatter::TryEnterAttribute(FArchiveFieldName AttributeName, bool bEnterWhenSaving)
{
	EnterAttribute(AttributeName);
	return true;
}

bool FTextOutputFormatter::TryEnterAttributedValueValue()
{
	EnterAttributedValueValue();
	return true;
}

void FTextOutputFormatter::Serialize(uint8& Value)
{
	GetValueBuffer() << Value;
}
void FTextOutputFormatter::Serialize(uint16& Value)
{
	GetValueBuffer() << Value;
}
void FTextOutputFormatter::Serialize(uint32& Value)
{
	GetValueBuffer() << Value;
}
void FTextOutputFormatter::Serialize(uint64& Value)
{
	GetValueBuffer() << Value;
}
void FTextOutputFormatter::Serialize(int8& Value)
{
	GetValueBuffer() << Value;
}
void FTextOutputFormatter::Serialize(int16& Value)
{
	GetValueBuffer() << Value;
}
void FTextOutputFormatter::Serialize(int32& Value)
{
	GetValueBuffer() << Value;
}
void FTextOutputFormatter::Serialize(int64& Value)
{
	GetValueBuffer() << Value;
}
void FTextOutputFormatter::Serialize(float& Value)
{
	GetValueBuffer().Appendf("%f", Value);
}
void FTextOutputFormatter::Serialize(double& Value)
{
	GetValueBuffer().Appendf("%f", Value);
}
void FTextOutputFormatter::Serialize(bool& Value)
{
	GetValueBuffer() << (Value ? UTF8TEXTVIEW("true") : UTF8TEXTVIEW("false"));
}
void FTextOutputFormatter::Serialize(UTF32CHAR& Value)
{
	GetValueBuffer() << Value;
}
void FTextOutputFormatter::Serialize(FString& Value)
{
	if (LooksLikeBinary(FStringView(Value)))
	{
		Serialize((void*)*Value, Value.Len() * sizeof(TCHAR));
		return;
	}

	FBuffer& Buffer = GetValueBuffer();
	FStringView View = FStringView(Value).Left(MaxStringLength);
	Buffer << "\"";
	if (NeedsEscaping(View))
	{
		WriteEscaped(Buffer, View);
	}	
	else
	{
		Buffer << View;
	}
	Buffer << UTF8TEXTVIEW("\"");
	if (View.Len() < Value.Len())
	{
		Buffer << " ...";
	}
}
void FTextOutputFormatter::Serialize(FName& Value)
{
	GetValueBuffer() << Value;
}
void FTextOutputFormatter::Serialize(UObject*& Value)
{
	FSoftObjectPath Path(Value);
	Serialize(Path);
}
void FTextOutputFormatter::Serialize(FText& Value)
{
	GetValueBuffer() << UTF8TEXTVIEW("\"") << Value.ToString() << UTF8TEXTVIEW("\"");
}
void FTextOutputFormatter::Serialize(FWeakObjectPtr& Value)
{
	FSoftObjectPath Path;
	if (UObject* Obj = Value.Get())
	{
		Path = FSoftObjectPath(Obj);
	}
	Serialize(Path);
}
void FTextOutputFormatter::Serialize(FSoftObjectPtr& Value)
{
	FSoftObjectPath Path = Value.GetUniqueID();
	Serialize(Path);
}
void FTextOutputFormatter::Serialize(FSoftObjectPath& Value)
{
	GetValueBuffer() << WriteToString<FName::StringBufferSize>(Value);
}
void FTextOutputFormatter::Serialize(FLazyObjectPtr& Value)
{
	FString S = Value.GetUniqueID().ToString();
	Serialize(S);
}
void FTextOutputFormatter::Serialize(FObjectPtr& Value)
{
	FSoftObjectPath Path = Value.GetPathName();
	Serialize(Path);
}
void FTextOutputFormatter::Serialize(TArray<uint8>& Value)
{
	Serialize((void*)Value.GetData(), Value.Num());
}
void FTextOutputFormatter::Serialize(void* Data, uint64 DataSize)
{
	FSHAHash Hash;
	FSHA1::HashBuffer(Data, DataSize, Hash.Hash);
	GetValueBuffer() << UTF8TEXTVIEW("BINARY ") << DataSize << UTF8TEXTVIEW(" bytes. SHA1 Hash: ");
	UE::String::BytesToHex(Hash.Hash, GetValueBuffer());
}

struct FSubcommand_PackageInfo
{
	FSharedParameters& Shared;
	TArray<FString> PackagePaths;
	bool bRecursive = false;
	bool bStdIn = false;
	bool bWaitForDebugger = false;

	FWildcardString Filter;
	FString OutPath;
	bool bOutputToSubdirectories = false;
	
	bool bAll = false;
	bool bSummary = false;
	bool bNames = false;
	bool bSoftPaths = false;
	bool bSoftPackageReferences = false;
	bool bImports = false;
	bool bExports = false;
	bool bText = false;
	bool bSimple = false;
	bool bDepends = false;
	bool bPaths = false;
	bool bThumbnails = false;
	bool bLazy = false;
	bool bAssetRegistry = false;
	
	FSubcommand_PackageInfo(FSharedParameters& InShared, CLI::App* InApp)
		: Shared(InShared)
	{
		CLI::App* Sub = InApp->add_subcommand("PackageInfo", "Print information contained within asset files.")
		 	// Allow global options to fall through and be parsed
			->fallthrough()
			->preparse_callback([this](std::size_t)
			{
				 PackagePaths.Reset(); 
				 Filter.Reset();
			})
			->parse_complete_callback([this]() { Main(); } );
		
		Sub->add_flag("--debug,--wait-for-debugger", bWaitForDebugger, "Wait for a debugger to be attached before continuing");
		
		CLI::Option_group* InputGroup = Sub->add_option_group("Input", "Where to get package data from");
		CLI::Option* PathOption = InputGroup->add_option("-p,--path", "Paths to packages or directories to read." LINE_TERMINATOR_ANSI "Relative paths are treated relative to current working directory.")
			->multi_option_policy(CLI::MultiOptionPolicy::TakeAll)
			->each([this](const std::string& s)
			{
				PackagePaths.Emplace(ConvertPathParameter(s));
			});
		InputGroup->add_flag("-r,--recursive", bRecursive, "Whether to scan directories recursively")
			->needs(PathOption);
		InputGroup->add_flag("--stdin", bStdIn, "Whether to read an asset from standard input");
		InputGroup->required(1);

		CLI::Option* OptionOut = Sub->add_option("-o,--out", "Path of file to write output to")
			->each([this](const std::string& s){ OutPath = ConvertPathParameter(s);});
		Sub->add_flag("--output-subdirectories", bOutputToSubdirectories, "Write output files to a path structure matching the package names.")
			->needs(OptionOut);
		
		Sub->add_option("-f,--filter", "Wilcard filter to apply to imports, exports, depends map, asset registry outputs")
			->expected(0,1)
			->each([this](const std::string& s)
			{
				Filter = UTF8_TO_TCHAR(s.c_str());
			});
		
		CLI::Option_group* OutputGroup = Sub->add_option_group("Sections", "Which parts of the package to output information about");
		OutputGroup->add_flag("--all", bAll, "Write out all package data tables");
		OutputGroup->add_flag("--summary", bSummary, "Write out the contents of the package file header");
		OutputGroup->add_flag("--names", bNames, "Write out the contents of the name table");
		OutputGroup->add_flag("--softpaths", bSoftPaths, "Write out the contents of the soft object path table");
		OutputGroup->add_flag("--softpackagerefs", bSoftPackageReferences, "Write out the contents of the soft package reference table");
		OutputGroup->add_flag("--imports", bImports, "Write out the objects imported from other packages by this package");
		OutputGroup->add_flag("--exports", bExports, "Write out the objects contained within this package");
		OutputGroup->add_flag("--depends", bDepends, "Write out the contents of the export dependency map");
		OutputGroup->add_flag("--text", bText, "Write out the contents of the gatherable text (localization) table");
		OutputGroup->add_flag("--simple", bSimple, "Write a reduced set of information where possible");
		OutputGroup->add_flag("--thumbnails", bThumbnails, "Write out the contents of the thumbnail table");
		OutputGroup->add_flag("--assetregistry", bAssetRegistry, "Write out the contenst of the asset registry data (tags etc) in the package.");
		OutputGroup->required(1);
	}
	
	void Main()
	{
		// If "-waitforattach" or "-WaitForDebugger" was specified, halt startup and wait for a debugger to attach before continuing
		if (bWaitForDebugger)
		{
			while (!FPlatformMisc::IsDebuggerPresent())
			{
				FPlatformProcess::Sleep(0.1f);
			}
			UE_DEBUG_BREAK();
		}

		if (!Filter.IsEmpty() && !Filter.ContainsWildcards(*Filter))
		{
			Filter = FString::Printf(TEXT("*%s*"), *Filter);
		}

		TArray<FString> AllPackagePaths;
		AllPackagePaths.Append(GatherAssetsInPaths(PackagePaths, bRecursive)); 
		
		if (bStdIn)
		{
			AllPackagePaths.Add(FString{});
		}
		
		IFileManager& FM = IFileManager::Get();
		if (OutPath.Len() != 0 && AllPackagePaths.Num() > 1 && !FM.DirectoryExists(*OutPath))
		{
			if (FM.FileExists(*OutPath))
			{
				UE_LOGF(LogUnrealPackageTool, Error, "Cannot output multiple packages to directory %ls, a file with that name exists", *OutPath);
				return;
			}
			else if (!IFileManager::Get().MakeDirectory(*OutPath, true))
			{
				UE_LOGF(LogUnrealPackageTool, Error, "Failed to create output directory %ls", *OutPath);
				return;
			}
		}
		
		TArray<uint8> StdInBuffer;
		const FString Extension = Shared.bJSON ? TEXT(".json") : TEXT(".txt");
		for (FString& PackagePath : AllPackagePaths)
		{
			FPackageReader Reader;
			FPackageReader::EOpenPackageResult ErrorCode;
			
			if (PackagePath.Len() == 0)
			{
				// Read from standard input
				StdInBuffer.Reset();
#if PLATFORM_WINDOWS
				_setmode(_fileno(stdin), _O_BINARY);
#else
				freopen(nullptr, "rb", stdin);
#endif
				while (!feof(stdin) && !ferror(stdin))
				{
					StdInBuffer.Reserve(StdInBuffer.Num() + 1024 * 1024);
					SIZE_T AmtRead = fread(StdInBuffer.GetData() + StdInBuffer.Num(), 1, StdInBuffer.Max() - StdInBuffer.Num(), stdin);
					StdInBuffer.AddUninitialized(AmtRead);
					if (AmtRead == 0)
					{
						break;
					}
				}
				TUniquePtr<FMemoryReader> Ar = MakeUnique<FMemoryReader>(StdInBuffer);	
				Reader.OpenPackageFile(MoveTemp(Ar), &ErrorCode);
			}
			else
			{
				FPaths::NormalizeFilename(PackagePath);
				Reader.OpenPackageFile(FStringView(PackagePath), &ErrorCode);
			}
			
			bool bContinue = true;
			TSet<FGuid> MissingCustomVersions;
			switch(ErrorCode)
			{
				case FPackageReader::EOpenPackageResult::Success:
					break;
				case FPackageReader::EOpenPackageResult::CustomVersionMissing:
					{
						FCustomVersionArray Versions = Reader.GetPackageFileSummary().GetCustomVersionContainer().GetAllVersions();
						for (const FCustomVersion& Version : Versions)
						{
							TOptional<FCustomVersion> CurrentVersion = FCurrentCustomVersions::Get(Version.Key);
							if (!CurrentVersion.IsSet())
							{
								UE_LOGF(LogUnrealPackageTool, Verbose, "Continuing to load package %ls with missing custom version %ls", *PackagePath, *WriteToString<128>(Version.Key));
								MissingCustomVersions.Add(Version.Key);
							}
						}
					}
					break;
				default:
					bContinue = false;
					break;
			}

			if (!bContinue)
			{
				UE_LOGF(LogUnrealPackageTool, Error, "Error opening package file %ls: %ls", *PackagePath, LexToString(ErrorCode)); 
				continue;
			}
			
			TUniquePtr<FArchive> Output;
			if (OutPath.Len() == 0)
			{
				Output.Reset(new FArchiveStdOut);
			}
			else 
			{
				FString OutFilePath;
				FString InputPath = Reader.GetLongPackageName().IsEmpty() ? PackagePath.Replace(TEXT("\\"), TEXT("/")) : Reader.GetLongPackageName();
				if (AllPackagePaths.Num() == 1 && !FM.DirectoryExists(*OutPath))
				{
					OutFilePath = OutPath;
				}
				else if(bOutputToSubdirectories)
				{
					// Remove drive letter if necessary
					if (InputPath.Len() > 2 && FChar::ToUpper(InputPath[0])!=FChar::ToLower(InputPath[0]) && InputPath[1]==TEXT(':'))
					{
						InputPath.RightChopInline(2);
					}
					OutFilePath = OutPath / InputPath;
				}
				else 
				{
					// Construct a path from the disk path	
					FString Path = InputPath;
					Path.ReplaceCharInline('/', '_');
					Path.ReplaceCharInline(':', '_');
					OutFilePath = OutPath / Path + Extension;
				}

				Output.Reset(FM.CreateFileWriter(*OutFilePath));
				if (!Output.IsValid())
				{
					UE_LOGF(LogUnrealPackageTool, Error, "Error opening output file %ls", *OutFilePath);
					continue;
				}

				UTF8CHAR UTF8BOM[] = { (UTF8CHAR)0xEF, (UTF8CHAR)0xBB, (UTF8CHAR)0xBF };
				Output->Serialize( &UTF8BOM, UE_ARRAY_COUNT(UTF8BOM) * sizeof(UTF8CHAR) );
			}
			
			TUniquePtr<FStructuredArchiveFormatter> Formatter;
			if (Shared.bJSON)
			{
				Formatter.Reset(new FJsonArchiveOutputFormatter(*Output));	
			}
			else
			{
				Formatter.Reset(new FTextOutputFormatter(*Output));	
			}
			FStructuredArchive Writer(*Formatter.Get());

			FPackageFileSummary Summary = Reader.GetPackageFileSummary();
			FStructuredArchiveRecord Root = Writer.Open().EnterRecord();
			
			if (bAll || bSummary)
			{
				Root.EnterField(TEXT("PackageFileSummary")) << Summary;
			}

			if (bAll || bNames)
			{
				TArray<FName> Names;
				if (Reader.GetNames(Names))
				{
					Root << SA_VALUE(TEXT("Names"), Names);
				}
				else
				{
					UE_LOGF(LogUnrealPackageTool, Error, "Error reading name table for package file %ls", *PackagePath);
				}
			}

			if (bAll || bSoftPaths )
			{
				TArray<FSoftObjectPath> Paths;
				if (Reader.GetSoftObjectPaths(Paths))
				{
					Root << SA_VALUE(TEXT("SoftObjectPaths"), Paths);
				}
				else
				{
					UE_LOGF(LogUnrealPackageTool, Error, "Error reading soft object path table for package file %ls", *PackagePath);
				}
			}
			
			FLinkerTables Tables;
			if (bAll || bImports || bExports || bSoftPackageReferences || bDepends)
			{
				if (!Reader.GetImports(Tables.ImportMap))
				{
					UE_LOGF(LogUnrealPackageTool, Error, "Error reading import table for package file %ls", *PackagePath);
				}
				if (!Reader.GetExports(Tables.ExportMap))
				{
					UE_LOGF(LogUnrealPackageTool, Error, "Error reading export table for package file %ls", *PackagePath);
				}
				if (!Reader.GetDependsMap(Tables.DependsMap))
				{
					UE_LOGF(LogUnrealPackageTool, Error, "Error reading depends map for package file %ls", *PackagePath);
				}
				if (!Reader.GetSoftPackageReferenceList(Tables.SoftPackageReferenceList))
				{
					UE_LOGF(LogUnrealPackageTool, Error, "Error reading soft package reference list for package file %ls", *PackagePath);
				}
			}
			
			if (bAll || bImports )
			{
				FStructuredArchiveStream Array = Root.EnterField(TEXT("Imports")).EnterStream();
				for (int32 ImportIndex = 0; ImportIndex < Tables.ImportMap.Num(); ++ImportIndex)
				{
					FObjectImport& Import = Tables.ImportMap[ImportIndex];
					FSoftObjectPath ImportPathName = Tables.GetImportPathName(ImportIndex);
					if (!Filter.IsEmpty() && !Filter.IsMatch(*WriteToString<FName::StringBufferSize>(ImportPathName)))
					{
						continue;
					}
					FStructuredArchiveRecord R = Array.EnterElement().EnterRecord();
					
					if (bSimple)
					{
						R << SA_VALUE(TEXT("PathName"), ImportPathName);
						FSoftObjectPath ClassPath{ FTopLevelAssetPath(Import.ClassPackage, Import.ClassName) };
						R << SA_VALUE(TEXT("ClassPath"), ClassPath);
					}
					else
					{
						R << SA_VALUE(TEXT("ObjectName"), Import.ObjectName);
						R << SA_VALUE(TEXT("PathName"), ImportPathName);
						R << SA_VALUE(TEXT("OuterIndex"), Import.OuterIndex);
						FSoftObjectPath OuterPathName = Tables.GetImportPathName(Import.OuterIndex);
						R << SA_VALUE(TEXT("Outer"), OuterPathName);
						FSoftObjectPath ClassPath{ FTopLevelAssetPath(Import.ClassPackage, Import.ClassName) };
						R << SA_VALUE(TEXT("ClassPath"), ClassPath);
						R << SA_VALUE(TEXT("Optional"), Import.bImportOptional);
					}
				}	
			}
			
			FString RootPackageName = Reader.GetLongPackageName();
			auto PackageIndexToObjectPath = [&Tables, &RootPackageName](FPackageIndex Index) -> FSoftObjectPath
			{
				if (Index.IsNull())
				{
					return FSoftObjectPath{};
				}

				return Index.IsExport() 
					? Tables.GetExportPathName(RootPackageName, Index.ToExport()) 
					: Tables.GetImportPathName(Index.ToImport());
			};
			if (bAll || bExports)
			{
				int32 NumExports = Tables.ExportMap.Num();
				FStructuredArchiveStream Array = Root.EnterField(TEXT("Exports")).EnterStream();
				for (int32 ExportIndex = 0; ExportIndex < Tables.ExportMap.Num(); ++ExportIndex)
				{
					FObjectExport& Export = Tables.ExportMap[ExportIndex];
					FSoftObjectPath ExportPathName = Tables.GetExportPathName(RootPackageName, ExportIndex);
					if (!Filter.IsEmpty() && !Filter.IsMatch(*WriteToString<FName::StringBufferSize>(ExportPathName)))
					{
						continue;
					}
					FStructuredArchiveRecord R = Array.EnterElement().EnterRecord();
					if (bSimple)
					{
						R << SA_VALUE(TEXT("Path"), ExportPathName);
						FSoftObjectPath ClassPathName = PackageIndexToObjectPath(Export.ClassIndex);
						R << SA_VALUE(TEXT("Class"), ClassPathName);
						FString ObjectFlags = LexToString(Export.ObjectFlags);
						R << SA_VALUE(TEXT("ObjectFlags"), ObjectFlags); 
						R << SA_VALUE(TEXT("SerialSize"), Export.SerialSize);
						R << SA_VALUE(TEXT("SerialOffset"), Export.SerialOffset);
					}
					else
					{
						R << SA_VALUE(TEXT("ObjectName"), Export.ObjectName);
						R << SA_VALUE(TEXT("OuterIndex"), Export.OuterIndex);
						R << SA_VALUE(TEXT("Path"), ExportPathName);
						R << SA_VALUE(TEXT("ClassIndex"), Export.ClassIndex);
						FSoftObjectPath ClassPathName = PackageIndexToObjectPath(Export.ClassIndex);
						R << SA_VALUE(TEXT("Class"), ClassPathName);
						R << SA_VALUE(TEXT("SuperIndex"), Export.SuperIndex);
						FSoftObjectPath SuperPathName = PackageIndexToObjectPath(Export.SuperIndex);
						R << SA_VALUE(TEXT("Super"), SuperPathName);
						R << SA_VALUE(TEXT("TemplateIndex"), Export.TemplateIndex);
						FSoftObjectPath TemplatePathName = PackageIndexToObjectPath(Export.TemplateIndex);
						R << SA_VALUE(TEXT("Template"), TemplatePathName);
						FString ObjectFlags = LexToString(Export.ObjectFlags);
						R << SA_VALUE(TEXT("ObjectFlags"), ObjectFlags); 
						R << SA_VALUE(TEXT("HashNext"), Export.HashNext);
						R << SA_VALUE(TEXT("SerialSize"), Export.SerialSize);
						R << SA_VALUE(TEXT("SerialOffset"), Export.SerialOffset);
						R << SA_VALUE(TEXT("ScriptSerializationStartOffset"), Export.ScriptSerializationStartOffset);
						R << SA_VALUE(TEXT("ScriptSerializationEndOffset"), Export.ScriptSerializationEndOffset);
						uint8 bForcedExport = Export.bForcedExport;
						R << SA_VALUE(TEXT("bForcedExport"), bForcedExport);
						uint8 bNotForClient = Export.bNotForClient; 
						R << SA_VALUE(TEXT("bNotForClient"), bNotForClient);
						uint8 bNotForServer = Export.bNotForServer;
						R << SA_VALUE(TEXT("bNotForServer"), bNotForServer);
						uint8 bNotAlwaysLoadedForEditorGame = Export.bNotAlwaysLoadedForEditorGame;
						R << SA_VALUE(TEXT("bNotAlwaysLoadedForEditorGame"), bNotAlwaysLoadedForEditorGame);
						uint8 bIsAsset = Export.bIsAsset;
						R << SA_VALUE(TEXT("bIsAsset"), bIsAsset);
						uint8 bIsInheritedInstance = Export.bIsInheritedInstance;
						R << SA_VALUE(TEXT("bIsInheritedInstance"), bIsInheritedInstance);
						uint8 bGeneratePublicHash = Export.bGeneratePublicHash;
						R << SA_VALUE(TEXT("bGeneratePublicHash"), bGeneratePublicHash);
					}
				}
			}
			
			if (bAll || bDepends)
			{
				int32 NumExports = Tables.ExportMap.Num();
				for (int32 ExportIndex = 0; ExportIndex < Tables.ExportMap.Num(); ++ExportIndex)
				{
					if (!Filter.IsEmpty() && !Filter.IsMatch(*WriteToString<FName::StringBufferSize>(PackageIndexToObjectPath(FPackageIndex::FromExport(ExportIndex)))))
					{
						--NumExports;
					}
				}

				FStructuredArchiveMap Map = Root.EnterField(TEXT("DependsMap")).EnterMap(NumExports);
				for (int32 ExportIndex = 0; ExportIndex < Tables.ExportMap.Num(); ++ExportIndex)
				{
					FString ExportPath = PackageIndexToObjectPath(FPackageIndex::FromExport(ExportIndex)).ToString();
					if (!Filter.IsEmpty() && !Filter.IsMatch(*ExportPath))
					{
						continue;
					}
					int32 NumDepends = Tables.DependsMap[ExportIndex].Num();
					if (NumDepends == 0)
					{
						continue;
					}
					if (bSimple)
					{
						FStructuredArchiveArray Array = Map.EnterElement(ExportPath).EnterArray(NumDepends);
						for (int32 DependsIndex = 0; DependsIndex < NumDepends; ++DependsIndex)
						{
							FSoftObjectPath DependsPath = PackageIndexToObjectPath(Tables.DependsMap[ExportIndex][DependsIndex]);
							Array.EnterElement() << DependsPath;
						}
					}	
					else
					{
						FStructuredArchiveArray Array = Map.EnterElement(ExportPath).EnterArray(NumDepends);
						for (int32 DependsIndex = 0; DependsIndex < NumDepends; ++DependsIndex)
						{
							FSoftObjectPath DependsPath = PackageIndexToObjectPath(Tables.DependsMap[ExportIndex][DependsIndex]);
							FStructuredArchiveRecord R = Array.EnterElement().EnterRecord();
							R << SA_VALUE(TEXT("Path"), DependsPath);
							R << SA_VALUE(TEXT("Index"), DependsIndex);
						}
					}
				}
			}
			
			if (bAll || bSoftPackageReferences)
			{
				TArray<FName> SoftPackageReferences;
				if (Reader.GetSoftPackageReferenceList(SoftPackageReferences))
				{
					Root << SA_VALUE(TEXT("SoftPackageReferences"), SoftPackageReferences);
				}
			}

			if (bAll || bText)
			{
				TArray<FGatherableTextData> GatherableTextMap;
				if (Reader.GetGatherableTextData(GatherableTextMap))
				{
					Root << SA_VALUE(TEXT("GatherableTextMap"), GatherableTextMap);
				}
				else
				{
					UE_LOGF(LogUnrealPackageTool, Error, "Error reading gatherable text data map for package file %ls", *PackagePath);
				}
			}

			if (bAll || bThumbnails)
			{
				TArray<FObjectFullNameAndThumbnail> Thumbnails;
				if (Reader.GetThumbnails(Thumbnails))
				{
					int32 NumThumbnails = Thumbnails.Num();
					FStructuredArchiveArray Array = Root.EnterField(TEXT("Thumbnails")).EnterArray(NumThumbnails);
					for (FObjectFullNameAndThumbnail& Thumbnail : Thumbnails)
					{
						FStructuredArchiveRecord R = Array.EnterElement().EnterRecord();
						R << SA_VALUE(TEXT("ObjectFullName"), Thumbnail.ObjectFullName);
						R << SA_VALUE(TEXT("FileOffset"), Thumbnail.FileOffset);
					}
				}
				else
				{
					UE_LOGF(LogUnrealPackageTool, Error, "Error reading thumbnail map package file %ls", *PackagePath);
				}
			}
			
			if (bAll || bAssetRegistry)
			{
				TArray<FAssetData*> AssetDatas;
				bool bCookedWithoutAssetData = false;
				if (Reader.ReadAssetRegistryData(AssetDatas, bCookedWithoutAssetData))
				{
					FStructuredArchiveStream Array = Root.EnterField(TEXT("AssetRegistry")).EnterStream();
					for (FAssetData* AssetData : AssetDatas)
					{
						if (!Filter.IsEmpty() && !Filter.IsMatch(*WriteToString<FName::StringBufferSize>(AssetData->AssetName)))
						{
							continue;
						}

						// Note: This could be replaced if/when structured archive serialization is implemented for FAssetData
						FStructuredArchiveRecord R = Array.EnterElement().EnterRecord();
						R << SA_VALUE(TEXT("PackageName"), AssetData->PackageName);
						R << SA_VALUE(TEXT("PackagePath"), AssetData->PackagePath);
						R << SA_VALUE(TEXT("AssetName"), AssetData->AssetName);
						FSoftObjectPath AssetClassPath{ AssetData->AssetClassPath, FString{} };
						R << SA_VALUE(TEXT("AssetClassPath"), AssetClassPath);
						R << SA_VALUE(TEXT("PackageFlags"), AssetData->PackageFlags);
						FSoftObjectPath OuterPathName = AssetData->GetOptionalOuterPathName().ToString();
						R << SA_VALUE(TEXT("OptionalOuterPath"), OuterPathName);
						TArray<int32> ChunkIDs {AssetData->GetChunkIDs()};
						R << SA_VALUE(TEXT("ChunkIDs"), ChunkIDs);
						
						int32 NumTags = AssetData->TagsAndValues.Num();
						FStructuredArchiveMap Tags = R.EnterField(TEXT("TagsAndValues")).EnterMap(NumTags);
						for (const TPair<FName, FAssetTagValueRef>& TagAndValue : AssetData->TagsAndValues)
						{
							FString Key = TagAndValue.Key.ToString();
							FString Value = TagAndValue.Value.AsString();	
							Tags.EnterElement(Key) << Value;
						}

						int32 NumBundles = AssetData->TaggedAssetBundles.IsValid() ? AssetData->TaggedAssetBundles->Bundles.Num() : 0;
						FStructuredArchiveMap Bundles = R.EnterField(TEXT("TaggedAssetBundles")).EnterMap(NumBundles);
						if (AssetData->TaggedAssetBundles.IsValid())
						{
							for (FAssetBundleEntry& Bundle : AssetData->TaggedAssetBundles->Bundles)
							{
								FString BundleName = Bundle.BundleName.ToString();
								Bundles.EnterElement(BundleName) << Bundle.AssetPaths;
							}
						}
						delete AssetData;
					}	
				}
				else
				{
					UE_LOGF(LogUnrealPackageTool, Error, "Error reading asset registry data from package file %ls", *PackagePath);
				}
			}
		}
	}
};

} // UE::PackageTool

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	using namespace UE::PackageTool;
	int32 Ret = GEngineLoop.PreInit(ArgC, ArgV);

	// Disable all logging because we want to output to stdout
	FSelfRegisteringExec::StaticExec(nullptr, TEXT("log logunrealpackagetool only"), *GLog);
	FSelfRegisteringExec::StaticExec(nullptr, TEXT("log logunrealpackagetool log"), *GLog);

	CLI::App App(
		"Utility for reading and modifying with Unreal asset files outside of the editor/engine.\n" \
		"Copyright Epic Games, Inc. All Rights Reserved.\n",

		"UnrealPackageTool"
	);

	TUniquePtr<FArchiveStdOut> Out = MakeUnique<FArchiveStdOut>();
	try 
	{
		App.ignore_case();
		App.set_help_flag();	
		App.set_help_all_flag("--help,-h", "Expand all help");	

		FSharedParameters Params(&App);
		FSubcommand_LicenseeVersionIsError LicenseeVersionIsError(Params, &App);
		FSubcommand_PackageInfo PackageInfo(Params, &App);
		for (FSubcommand* Sub = SubcommandList; Sub != nullptr; Sub = Sub->Next)
		{
			Sub->RegisterSubcommand(&Params, &App);
		}

		if (Ret == 0)
		{
			App.parse();
			if (App.get_subcommands().size() == 0)
			{
				Out->Logf(TEXT("%s"), UTF8_TO_TCHAR(App.help().c_str()));
			}
		}
	}
    catch (CLI::CallForAllHelp& e) 
	{
		Out->Logf(TEXT("%s"), UTF8_TO_TCHAR(App.help("", CLI::AppFormatMode::All).c_str()));
		Ret = e.get_exit_code();	
    }
	catch (CLI::CallForHelp& e)
	{
		Out->Logf(TEXT("%s"), UTF8_TO_TCHAR(App.help().c_str()));
		Ret = e.get_exit_code();	
	}
	catch (CLI::RequiredError& e)
	{
		Out->Logf(TEXT("Error: %s"), UTF8_TO_TCHAR(e.what()));
		Out->Logf(TEXT("%s"), UTF8_TO_TCHAR(App.help().c_str()));
		Ret = e.get_exit_code();	
	}
	catch (CLI::Error& e)
	{
		Out->Logf(TEXT("Error: %s"), UTF8_TO_TCHAR(e.what()));
		Ret = e.get_exit_code();
	}
	catch(...)
	{
		Out->Logf(TEXT("Unknown error"));
		Ret = 1;
	}

	RequestEngineExit(TEXT("Exiting"));
	FEngineLoop::AppPreExit();
	FModuleManager::Get().UnloadModulesAtShutdown();
	FEngineLoop::AppExit();

	return Ret;
}
