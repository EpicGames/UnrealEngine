// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"

#include <string>

namespace CLI
{
	class App;
}


namespace UE::PackageTool
{
// Parameters shared between multiple execution modes
struct FSharedParameters
{
	bool bJSON = false;

	FSharedParameters(CLI::App* InApp);
};

// Base class for a subcommand which should be registered as part of UnrealPackageTool to add argument parsing to CLI::App
struct FSubcommand
{
	FSubcommand();
	virtual ~FSubcommand();
	UE_NONCOPYABLE(FSubcommand);

	virtual void RegisterSubcommand(FSharedParameters* InShared, CLI::App*) = 0;

	FSubcommand* Next = nullptr;
};


// Convert a utf8 command line parameter to an FString path, treating relative paths as relative to the working directory rather than the engine base dir
FString ConvertPathParameter(const std::string& InParam);

// Utility archive for printing json/package info to stdout
struct FArchiveStdOut : public FArchive
{
	int64 Pos = 0;

	~FArchiveStdOut();
	
	// Both formatter types provide utf8 text
	virtual void Serialize(void* Data, int64 Len) override;
	
	// Required for correct formatting of JSON output
	virtual int64 Tell() override;
};

// Structured archive formatter to print column-aligned human readable data.
// Allows us to share driving code with JSON output
// The output of this formatter is not indended for parsing/machine consumption
class FTextOutputFormatter final : public FStructuredArchiveFormatter
{
	using FBuffer = TUtf8StringBuilder<2048>;
	static const constexpr int32 IndentWidth = 4;
	static const constexpr int32 MaxStringLength = 256;

	struct FStackEntry
	{
		int32 Indent = 0;
		TArray<TTuple<FString, FString>> MapEntries;
		TUniquePtr<FBuffer> ValueBuffer;
	};
	FArchive& Ar;
	TArray<FStackEntry> Stack;
	
	int32 GetIndent();
	FBuffer& GetValueBuffer();

	void FlushBuffer(FBuffer& Buffer);
	static bool LooksLikeBinary(FStringView View);
	static bool CharNeedsEscaping(TCHAR C);
	static bool NeedsEscaping(FStringView View);
	static void WriteEscaped(FBuffer& Buffer, FStringView View);

public:
	FTextOutputFormatter(FArchive& InAr);
	virtual ~FTextOutputFormatter();
	virtual FArchive& GetUnderlyingArchive() override;
	virtual bool HasDocumentTree() const override;

	virtual void EnterRecord() override;
	virtual void LeaveRecord() override;
	virtual void EnterField(FArchiveFieldName Name) override;
	virtual void LeaveField() override;
	virtual bool TryEnterField(FArchiveFieldName Name, bool bEnterWhenSaving) override;
	virtual void EnterArray(int32& NumElements) override;
	virtual void LeaveArray() override;
	virtual void EnterArrayElement() override;
	virtual void LeaveArrayElement() override;
	virtual void EnterStream() override;
	virtual void LeaveStream() override;
	virtual void EnterStreamElement() override;
	virtual void LeaveStreamElement() override;
	virtual void EnterMap(int32& NumElements) override;
	virtual void LeaveMap() override;
	virtual void EnterMapElement(FString& Name) override;
	virtual void LeaveMapElement() override;
	virtual void EnterAttributedValue() override;
	virtual void EnterAttribute(FArchiveFieldName AttributeName) override;
	virtual void EnterAttributedValueValue() override;
	virtual void LeaveAttribute() override;
	virtual void LeaveAttributedValue() override;
	virtual bool TryEnterAttribute(FArchiveFieldName AttributeName, bool bEnterWhenSaving) override;
	virtual bool TryEnterAttributedValueValue() override;
	virtual void Serialize(uint8& Value) override;
	virtual void Serialize(uint16& Value) override;
	virtual void Serialize(uint32& Value) override;
	virtual void Serialize(uint64& Value) override;
	virtual void Serialize(int8& Value) override;
	virtual void Serialize(int16& Value) override;
	virtual void Serialize(int32& Value) override;
	virtual void Serialize(int64& Value) override;
	virtual void Serialize(float& Value) override;
	virtual void Serialize(double& Value) override;
	virtual void Serialize(bool& Value) override;
	virtual void Serialize(UTF32CHAR& Value) override;
	virtual void Serialize(FString& Value) override;
	virtual void Serialize(FName& Value) override;
	virtual void Serialize(UObject*& Value) override;
	virtual void Serialize(FText& Value) override;
	virtual void Serialize(FWeakObjectPtr& Value) override;
	virtual void Serialize(FSoftObjectPtr& Value) override;
	virtual void Serialize(FSoftObjectPath& Value) override;
	virtual void Serialize(FLazyObjectPtr& Value) override;
	virtual void Serialize(FObjectPtr& Value) override;
	virtual void Serialize(TArray<uint8>& Value) override;
	virtual void Serialize(void* Data, uint64 DataSize) override;
	
};
}